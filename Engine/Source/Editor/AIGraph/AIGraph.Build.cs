// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AIGraph : ModuleRules
{
	public AIGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		OverridePackageType = PackageOverrideType.EngineDeveloper;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AIModule",
				"ApplicationCore",
				"BlueprintGraph",
				"Core",
				"GraphEditor",
				"InputCore",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
			}
		);
	}
}