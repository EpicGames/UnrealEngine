// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "UObject/UObjectGlobals.h"

#include "AIGraph.generated.h"

#define UE_API AIGRAPH_API

class FArchive;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UAIGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	int32 GraphVersion;

	UE_API virtual void OnCreated();
	UE_API virtual void OnLoaded();
	UE_API virtual void Initialize();

	UE_API virtual void UpdateAsset(int32 UpdateFlags = 0);
	UE_API virtual void UpdateVersion();
	UE_API virtual void MarkVersion();

	UE_API virtual void OnSubNodeDropped();
	UE_API virtual void OnNodesPasted(const FString& ImportStr);

	UE_API bool UpdateUnknownNodeClasses();

	UE_DEPRECATED(5.6, "This method was only used to fetch error messages and log them. It has been renamed to better fit its purpose. Use UpdateErrorMessages instead.")
	UE_API void UpdateDeprecatedClasses();

	/** Sets error messages and logs errors about nodes. */
	UE_API void UpdateErrorMessages();
	UE_API void RemoveOrphanedNodes();
	UE_API void UpdateClassData();

	UE_API bool IsLocked() const;
	UE_API void LockUpdates();
	UE_API void UnlockUpdates();

	//~ Begin UObject Interface.
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

protected:

	/** if set, graph modifications won't cause updates in internal tree structure
	 *  flag allows freezing update during heavy changes like pasting new nodes 
	 */
	uint32 bLockUpdates : 1;

	UE_API virtual void CollectAllNodeInstances(TSet<UObject*>& NodeInstances);
	UE_API virtual bool CanRemoveNestedObject(UObject* TestObject) const;
	UE_API virtual void OnNodeInstanceRemoved(UObject* NodeInstance);

	UE_API UEdGraphPin* FindGraphNodePin(UEdGraphNode* Node, EEdGraphPinDirection Dir);
};

#undef UE_API
