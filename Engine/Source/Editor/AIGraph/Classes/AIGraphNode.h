// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "AIGraphTypes.h"
#include "AIGraphNode.generated.h"

#define UE_API AIGRAPH_API

class UEdGraph;
class UEdGraphPin;
class UEdGraphSchema;
struct FDiffResults;
struct FDiffSingleResult;

UCLASS(MinimalAPI)
class UAIGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** instance class */
	UPROPERTY()
	struct FGraphNodeClassData ClassData;

	UPROPERTY()
	TObjectPtr<UObject> NodeInstance;

	UPROPERTY(transient)
	TObjectPtr<UAIGraphNode> ParentNode;

	UPROPERTY()
	TArray<TObjectPtr<UAIGraphNode>> SubNodes;

	/** subnode index assigned during copy operation to connect nodes again on paste */
	UPROPERTY()
	int32 CopySubNodeIndex;

	/** if set, all modifications (including delete/cut) are disabled */
	UPROPERTY()
	uint32 bIsReadOnly : 1;

	/** if set, this node will be always considered as subnode */
	UPROPERTY()
	uint32 bIsSubNode : 1;

	/** error message for node */
	UPROPERTY()
	FString ErrorMessage;

	//~ Begin UEdGraphNode Interface
	UE_API virtual class UAIGraph* GetAIGraph();
	UE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void PrepareForCopying() override;
	UE_API virtual bool CanDuplicateNode() const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void NodeConnectionListChanged() override;
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	UE_API virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	UE_API virtual FString GetPropertyNameAndValueForDiff(const FProperty* Prop, const uint8* PropertyAddr) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditUndo() override;
#endif
	// End UObject

	// @return the input pin for this state
	UE_API virtual UEdGraphPin* GetInputPin(int32 InputIndex = 0) const;
	// @return the output pin for this state
	UE_API virtual UEdGraphPin* GetOutputPin(int32 InputIndex = 0) const;
	virtual UEdGraph* GetBoundGraph() const { return NULL; }

	UE_API virtual FText GetDescription() const;
	UE_API virtual void PostCopyNode();

	UE_API void AddSubNode(UAIGraphNode* SubNode, class UEdGraph* ParentGraph);
	UE_API void RemoveSubNode(UAIGraphNode* SubNode);
	UE_API virtual void RemoveAllSubNodes();
	UE_API virtual void OnSubNodeRemoved(UAIGraphNode* SubNode);
	UE_API virtual void OnSubNodeAdded(UAIGraphNode* SubNode);

	UE_API virtual int32 FindSubNodeDropIndex(UAIGraphNode* SubNode) const;
	UE_API virtual void InsertSubNodeAt(UAIGraphNode* SubNode, int32 DropIndex);

	/** check if node is subnode */
	UE_API virtual bool IsSubNode() const;

	/** initialize instance object  */
	UE_API virtual void InitializeInstance();

	/** reinitialize node instance */
	UE_API virtual bool RefreshNodeClass();

	/** updates ClassData from node instance */
	UE_API virtual void UpdateNodeClassData();

	/**
	 * Checks for any errors in this node and updates ErrorMessage with any resulting message
	 * Called every time the graph is serialized (i.e. loaded, saved, execution index changed, etc)
	 */
	UE_API virtual void UpdateErrorMessage();

	/** Check if node instance uses blueprint for its implementation */
	UE_API bool UsesBlueprint() const;

	/** check if node has any errors, used for assigning colors on graph */
	UE_API virtual bool HasErrors() const;

	static UE_API void UpdateNodeClassDataFrom(UClass* InstanceClass, FGraphNodeClassData& UpdatedData);

protected:

	UE_API virtual void ResetNodeOwner();
};

#undef UE_API
