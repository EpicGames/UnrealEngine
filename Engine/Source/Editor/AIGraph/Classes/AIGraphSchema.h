// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"
#include "AIGraphTypes.h"
#include "AIGraphSchema.generated.h"

#define UE_API AIGRAPH_API

class FSlateRect;
class UEdGraph;

/** Action to add a comment to the graph */
USTRUCT()
struct FAISchemaAction_AddComment : public FEdGraphSchemaAction
{
	GENERATED_BODY()
	
	FAISchemaAction_AddComment() : FEdGraphSchemaAction() {}
	FAISchemaAction_AddComment(FText InDescription, FText InToolTip)
		: FEdGraphSchemaAction(FText(), MoveTemp(InDescription), MoveTemp(InToolTip), 0)
	{
	}

	// FEdGraphSchemaAction interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override final;
	// End of FEdGraphSchemaAction interface
};

/** Action to add a node to the graph */
USTRUCT()
struct FAISchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UAIGraphNode> NodeTemplate;

	FAISchemaAction_NewNode()
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
	{}

	FAISchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodeTemplate(nullptr)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a subnode to the selected node */
USTRUCT()
struct FAISchemaAction_NewSubNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UAIGraphNode> NodeTemplate;

	/** parent node */
	UPROPERTY()
	TObjectPtr<class UAIGraphNode> ParentNode;

	FAISchemaAction_NewSubNode()
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
		, ParentNode(nullptr)
	{}

	FAISchemaAction_NewSubNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodeTemplate(nullptr)
		, ParentNode(nullptr)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End FEdGraphSchemaAction Interface
};

UCLASS(MinimalAPI)
class UAIGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphSchema Interface
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	UE_API virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	UE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	UE_API virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	//~ End EdGraphSchema Interface

	UE_API virtual void GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const;
	UE_API virtual void GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const;

protected:

	static UE_API TSharedPtr<FAISchemaAction_NewNode> AddNewNodeAction(FGraphActionListBuilderBase& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip);
	static UE_API TSharedPtr<FAISchemaAction_NewSubNode> AddNewSubNodeAction(FGraphActionListBuilderBase& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip);
};

#undef UE_API
