// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "AIGraphTypes.generated.h"

#define UE_API AIGRAPH_API

USTRUCT()
struct FGraphNodeClassData
{
	GENERATED_USTRUCT_BODY()

	FGraphNodeClassData() {}
	UE_API FGraphNodeClassData(UClass* InClass, const FString& InDeprecatedMessage);
	UE_API FGraphNodeClassData(const FTopLevelAssetPath& InGeneratedClassPath, UClass* InClass);
	UE_API FGraphNodeClassData(const FString& InAssetName, const FString& InGeneratedClassPackage, const FString& InClassName, UClass* InClass);

	UE_API FString ToString() const;
	UE_API FString GetClassName() const;
	UE_API FText GetCategory() const;
	UE_API FString GetDisplayName() const;
	UE_API FText GetTooltip() const;
	UE_API UClass* GetClass(bool bSilent = false);
	UE_API bool IsAbstract() const;

	FORCEINLINE bool IsBlueprint() const { return AssetName.Len() > 0; }
	FORCEINLINE bool IsDeprecated() const { return DeprecatedMessage.Len() > 0; }
	FORCEINLINE FString GetDeprecatedMessage() const { return DeprecatedMessage; }
	FORCEINLINE FString GetPackageName() const { return GeneratedClassPackage; }

	/** set when child class masked this one out (e.g. always use game specific class instead of engine one) */
	uint32 bIsHidden : 1;

	/** set when class wants to hide parent class from selection (just one class up hierarchy) */
	uint32 bHideParent : 1;

private:

	/** pointer to uclass */
	TWeakObjectPtr<UClass> Class;

	/** path to class if it's not loaded yet */
	UPROPERTY()
	FString AssetName;
	
	UPROPERTY()
	FString GeneratedClassPackage;

	/** resolved name of class from asset data */
	UPROPERTY()
	FString ClassName;

	/** User-defined category for this class */
	UPROPERTY()
	FText Category;

	/** message for deprecated class */
	FString DeprecatedMessage;
};

struct FGraphNodeClassNode
{
	FGraphNodeClassData Data;
	FString ParentClassName;

	TSharedPtr<FGraphNodeClassNode> ParentNode;
	TArray<TSharedPtr<FGraphNodeClassNode> > SubNodes;

	UE_API void AddUniqueSubNode(TSharedPtr<FGraphNodeClassNode> SubNode);
};

struct FGraphNodeClassHelper
{
	DECLARE_MULTICAST_DELEGATE(FOnPackageListUpdated);

	UE_API FGraphNodeClassHelper(UClass* InRootClass);
	UE_API ~FGraphNodeClassHelper();

	UE_API void GatherClasses(const UClass* BaseClass, TArray<FGraphNodeClassData>& AvailableClasses);
	static UE_API FString GetDeprecationMessage(const UClass* Class);

	UE_API void OnAssetAdded(const struct FAssetData& AssetData);
	UE_API void OnAssetRemoved(const struct FAssetData& AssetData);
	UE_API void InvalidateCache();
	UE_API void OnReloadComplete(EReloadCompleteReason Reason);

	static UE_API void AddUnknownClass(const FGraphNodeClassData& ClassData);
	static UE_API bool IsClassKnown(const FGraphNodeClassData& ClassData);
	static UE_API FOnPackageListUpdated OnPackageListUpdated;

	static UE_API int32 GetObservedBlueprintClassCount(UClass* BaseNativeClass);
	static UE_API void AddObservedBlueprintClasses(UClass* BaseNativeClass);
	UE_API void UpdateAvailableBlueprintClasses();

	/** Adds a single class to the list of hidden classes */
	UE_API void AddForcedHiddenClass(UClass* Class);

	/** Overrides all previously set hidden classes */
	UE_API void SetForcedHiddenClasses(const TSet<UClass*>& Classes);

	UE_API void SetGatherBlueprints(bool bGather);

private:

	UClass* RootNodeClass;
	TSharedPtr<FGraphNodeClassNode> RootNode;
	static UE_API TArray<FName> UnknownPackages;
	static UE_API TMap<UClass*, int32> BlueprintClassCount;
	TSet<UClass*> ForcedHiddenClasses;
	bool bGatherBlueprints = true;

	UE_API TSharedPtr<FGraphNodeClassNode> CreateClassDataNode(const struct FAssetData& AssetData);
	UE_API TSharedPtr<FGraphNodeClassNode> FindBaseClassNode(TSharedPtr<FGraphNodeClassNode> Node, const FString& ClassName);
	UE_API void FindAllSubClasses(TSharedPtr<FGraphNodeClassNode> Node, TArray<FGraphNodeClassData>& AvailableClasses);

	UE_API void BuildClassGraph();
	UE_API void AddClassGraphChildren(TSharedPtr<FGraphNodeClassNode> Node, TArray<TSharedPtr<FGraphNodeClassNode> >& NodeList);

	UE_API bool IsHidingClass(UClass* Class);
	UE_API bool IsHidingParentClass(UClass* Class);
	UE_API bool IsPackageSaved(FName PackageName);
};

#undef UE_API
