// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraph.h"

#include "AIGraphModule.h"
#include "AIGraphNode.h"
#include "AIGraphTypes.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformMath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIGraph)

UAIGraph::UAIGraph(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bLockUpdates = false;
}

void UAIGraph::UpdateAsset(int32 UpdateFlags)
{
	// empty in base class
}

void UAIGraph::OnCreated()
{
	MarkVersion();
}

void UAIGraph::OnLoaded()
{
	UpdateErrorMessages();
	UpdateUnknownNodeClasses();
}

void UAIGraph::Initialize()
{
	UpdateVersion();
}

void UAIGraph::UpdateVersion()
{
	if (GraphVersion == 1)
	{
		return;
	}

	MarkVersion();
	Modify();
}

void UAIGraph::MarkVersion()
{
	GraphVersion = 1;
}

bool UAIGraph::UpdateUnknownNodeClasses()
{
	bool bUpdated = false;
	for (int32 NodeIdx = 0; NodeIdx < Nodes.Num(); NodeIdx++)
	{
		UAIGraphNode* MyNode = Cast<UAIGraphNode>(Nodes[NodeIdx]);
		if (MyNode)
		{
			const bool bUpdatedNode = MyNode->RefreshNodeClass();
			bUpdated = bUpdated || bUpdatedNode;

			for (int32 SubNodeIdx = 0; SubNodeIdx < MyNode->SubNodes.Num(); SubNodeIdx++)
			{
				if (MyNode->SubNodes[SubNodeIdx])
				{
					const bool bUpdatedSubNode = MyNode->SubNodes[SubNodeIdx]->RefreshNodeClass();
					bUpdated = bUpdated || bUpdatedSubNode;
				}
			}
		}
	}

	return bUpdated;
}

void UpdateAIGraphNodeErrorMessage(UAIGraphNode& Node)
{
	// Broke out setting error message in to own function so it can be reused when iterating nodes collection.
	if (Node.NodeInstance)
	{
		Node.ErrorMessage = FGraphNodeClassHelper::GetDeprecationMessage(Node.NodeInstance->GetClass());

		// Only check for node-specific errors if the node is not deprecated
		if (Node.ErrorMessage.IsEmpty())
		{
			Node.UpdateErrorMessage();

			// For node-specific validation we don't want to spam the log with errors
			return;
		}
	}
	else
	{
		// Null instance. Do we have any meaningful class data?
		FString StoredClassName = Node.ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		if (!StoredClassName.IsEmpty())
		{
			// There is class data here but the instance was not be created.
			static const FString IsMissingClassMessage(" class missing. Referenced by ");
			Node.ErrorMessage = StoredClassName + IsMissingClassMessage + Node.GetFullName();
		}
	}

	if (Node.HasErrors())
	{
		UE_LOGF(LogAIGraph, Error, "%ls", *Node.ErrorMessage);
	}
}

// deprecated
void UAIGraph::UpdateDeprecatedClasses()
{
	UpdateErrorMessages();
}

void UAIGraph::UpdateErrorMessages()
{
	for (int32 Idx = 0, IdxNum = Nodes.Num(); Idx < IdxNum; ++Idx)
	{
		UAIGraphNode* Node = Cast<UAIGraphNode>(Nodes[Idx]);
		if (Node != nullptr)
		{
			UpdateAIGraphNodeErrorMessage(*Node);
			
			for (int32 SubIdx = 0, SubIdxNum = Node->SubNodes.Num(); SubIdx < SubIdxNum; ++SubIdx)
			{
				if (Node->SubNodes[SubIdx] != nullptr)
				{
					UpdateAIGraphNodeErrorMessage(*Node->SubNodes[SubIdx]);
				}
			}
		}
	}
}

void UAIGraph::Serialize(FArchive& Ar)
{
	// Overridden to flags up errors in the behavior tree while cooking.
	Super::Serialize(Ar);

	// Execute UpdateErrorMessages only when saving to persistent storage,
	// otherwise node instances might not be fully created (i.e. transaction buffer while loading the asset).
	if ((Ar.IsSaving() && Ar.IsPersistent()) || Ar.IsCooking())
	{
		// Logging of errors happens in UpdateErrorMessages
		UpdateErrorMessages();
	}
}

void UAIGraph::UpdateClassData()
{
	for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
	{
		UAIGraphNode* Node = Cast<UAIGraphNode>(Nodes[Idx]);
		if (Node)
		{
			Node->UpdateNodeClassData();

			for (int32 SubIdx = 0; SubIdx < Node->SubNodes.Num(); SubIdx++)
			{
				if (UAIGraphNode* SubNode = Node->SubNodes[SubIdx])
				{
					SubNode->UpdateNodeClassData();
				}
			}
		}
	}
}

void UAIGraph::CollectAllNodeInstances(TSet<UObject*>& NodeInstances)
{
	for (int32 Idx = 0; Idx < Nodes.Num(); Idx++)
	{
		UAIGraphNode* MyNode = Cast<UAIGraphNode>(Nodes[Idx]);
		if (MyNode)
		{
			NodeInstances.Add(MyNode->NodeInstance);

			for (int32 SubIdx = 0; SubIdx < MyNode->SubNodes.Num(); SubIdx++)
			{
				if (MyNode->SubNodes[SubIdx])
				{
					NodeInstances.Add(MyNode->SubNodes[SubIdx]->NodeInstance);
				}
			}
		}
	}
}

bool UAIGraph::CanRemoveNestedObject(UObject* TestObject) const
{
	return !TestObject->IsA(UEdGraphNode::StaticClass()) &&
		!TestObject->IsA(UEdGraph::StaticClass()) &&
		!TestObject->IsA(UEdGraphSchema::StaticClass());
}

void UAIGraph::RemoveOrphanedNodes()
{
	TSet<UObject*> NodeInstances;
	CollectAllNodeInstances(NodeInstances);

	NodeInstances.Remove(nullptr);

	// Obtain a list of all nodes actually in the asset and discard unused nodes
	TArray<UObject*> AllInners;
	GetObjectsWithOuter(GetOuter(), AllInners, EGetObjectsFlags::None);
	for (auto InnerIt = AllInners.CreateConstIterator(); InnerIt; ++InnerIt)
	{
		UObject* TestObject = *InnerIt;
		if (!NodeInstances.Contains(TestObject) && CanRemoveNestedObject(TestObject))
		{
			OnNodeInstanceRemoved(TestObject);

			TestObject->SetFlags(RF_Transient);
			TestObject->Rename(NULL, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}
}

void UAIGraph::OnNodeInstanceRemoved(UObject* NodeInstance)
{
	// empty in base class
}

void UAIGraph::OnNodesPasted(const FString& ImportStr)
{
	// empty in base class
}

UEdGraphPin* UAIGraph::FindGraphNodePin(UEdGraphNode* Node, EEdGraphPinDirection Dir)
{
	UEdGraphPin* Pin = nullptr;
	for (int32 Idx = 0; Idx < Node->Pins.Num(); Idx++)
	{
		if (Node->Pins[Idx]->Direction == Dir)
		{
			Pin = Node->Pins[Idx];
			break;
		}
	}

	return Pin;
}

bool UAIGraph::IsLocked() const
{
	return bLockUpdates;
}

void UAIGraph::LockUpdates()
{
	bLockUpdates = true;
}

void UAIGraph::UnlockUpdates()
{
	bLockUpdates = false;
	UpdateAsset();
}

void UAIGraph::OnSubNodeDropped()
{
	NotifyGraphChanged();
}
