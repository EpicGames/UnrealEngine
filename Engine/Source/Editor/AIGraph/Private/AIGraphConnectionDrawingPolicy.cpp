// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraphConnectionDrawingPolicy.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/PaintGeometry.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Optional.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SGraphNode.h"
#include "Styling/SlateBrush.h"

class FSlateRect;
class SWidget;
struct FGeometry;

FAIGraphConnectionDrawingPolicy::FAIGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
}

void FAIGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireThickness = 1.5f;

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}
}

void FAIGraphConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	// Build an acceleration structure to quickly find geometry for the nodes
	NodeWidgetMap.Empty();
	for (int32 NodeIndex = 0; NodeIndex < ArrangedNodes.Num(); ++NodeIndex)
	{
		FArrangedWidget& CurWidget = ArrangedNodes[NodeIndex];
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(CurWidget.Widget);
		NodeWidgetMap.Add(ChildNode->GetNodeObj(), NodeIndex);
	}

	// Now draw
	FConnectionDrawingPolicy::Draw(InPinGeometries, ArrangedNodes);
}

void FAIGraphConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin)
{
	FConnectionParams Params;
	DetermineWiringStyle(Pin, nullptr, /*inout*/ Params);

	if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		DrawSplineWithArrow(FGeometryHelper::FindClosestPointOnGeom(PinGeometry, EndPoint), EndPoint, Params);
	}
	else
	{
		DrawSplineWithArrow(FGeometryHelper::FindClosestPointOnGeom(PinGeometry, StartPoint), StartPoint, Params);
	}
}

void FAIGraphConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params)
{
	// bUserFlag1 indicates that we need to reverse the direction of connection (used by debugger)
	const FVector2f& P0 = Params.bUserFlag1 ? EndAnchorPoint : StartAnchorPoint;
	const FVector2f& P1 = Params.bUserFlag1 ? StartAnchorPoint : EndAnchorPoint;

	Internal_DrawLineWithArrow(P0, P1, Params);
}

void FAIGraphConnectionDrawingPolicy::Internal_DrawLineWithArrow(const UE::Slate::FDeprecateVector2DParameter& StartAnchorPoint, const UE::Slate::FDeprecateVector2DParameter& EndAnchorPoint, const FConnectionParams& Params)
{
	//@TODO: Should this be scaled by zoom factor?
	const float LineSeparationAmount = 4.5f;

	const FVector2f StartAnchorPointTemp(StartAnchorPoint);
	const FVector2f EndAnchorPointTemp(static_cast<float>(EndAnchorPoint.X), static_cast<float>(EndAnchorPoint.Y));
	const FVector2f DeltaPos = EndAnchorPointTemp - StartAnchorPointTemp;
	const FVector2f UnitDelta = DeltaPos.GetSafeNormal();
	const FVector2f Normal = FVector2f(DeltaPos.Y, -DeltaPos.X).GetSafeNormal();

	// Come up with the final start/end points
	const FVector2f DirectionBias = Normal * LineSeparationAmount;
	const FVector2f LengthBias = ArrowRadius.X * UnitDelta;
	const FVector2f StartPoint = StartAnchorPointTemp + DirectionBias + LengthBias;
	const FVector2f EndPoint = EndAnchorPointTemp + DirectionBias - LengthBias;

	// Draw a line/spline
	DrawConnection(WireLayerID, StartPoint, EndPoint, Params);

	// Draw the arrow
	const FVector2f ArrowDrawPos = EndPoint - ArrowRadius;
	const float AngleInRadians = static_cast<float>(FMath::Atan2(DeltaPos.Y, DeltaPos.X));

	FSlateDrawElement::MakeRotatedBox(
		DrawElementsList,
		ArrowLayerID,
		FPaintGeometry(ArrowDrawPos, ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
		ArrowImage,
		ESlateDrawEffect::None,
		AngleInRadians,
		TOptional<FVector2f>(),
		FSlateDrawElement::RelativeToElement,
		Params.WireColor
		);
}

void FAIGraphConnectionDrawingPolicy::DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params)
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2f StartCenter = FGeometryHelper::CenterOf(StartGeom);
	const FVector2f EndCenter = FGeometryHelper::CenterOf(EndGeom);
	const FVector2f SeedPoint = (StartCenter + EndCenter) * 0.5f;

	// Find the (approximate) closest points between the two boxes
	const FVector2f StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(StartGeom, SeedPoint);
	const FVector2f EndAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(EndGeom, SeedPoint);

	DrawSplineWithArrow(StartAnchorPoint, EndAnchorPoint, Params);
}

FVector2f FAIGraphConnectionDrawingPolicy::ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const
{
	const FVector2f Delta = End - Start;
	const FVector2f NormDelta = Delta.GetSafeNormal();

	return NormDelta;
}
