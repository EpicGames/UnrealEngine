// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraphEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/EditorEngine.h"
#include "EngineGlobals.h"
#include "AIGraph.h"
#include "AIGraphTypes.h"
#include "AIGraphNode.h"
#include "ScopedTransaction.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformApplicationMisc.h"
#include "EdGraph/EdGraphSchema.h"

#define LOCTEXT_NAMESPACE "AIGraph"

FAIGraphEditor::FAIGraphEditor()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor != NULL)
	{
		Editor->RegisterForUndo(this);
	}

	OnClassListUpdatedDelegateHandle = FGraphNodeClassHelper::OnPackageListUpdated.AddRaw(this, &FAIGraphEditor::OnClassListUpdated);
}

FAIGraphEditor::~FAIGraphEditor()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor)
	{
		Editor->UnregisterForUndo(this);
	}

	FGraphNodeClassHelper::OnPackageListUpdated.Remove(OnClassListUpdatedDelegateHandle);
}

void FAIGraphEditor::CreateCommandList()
{
	if (GraphEditorCommands.IsValid())
	{
		return;
	}

	GraphEditorCommands = MakeShareable(new FUICommandList);

	// Can't use CreateSP here because derived editor are already implementing TSharedFromThis<FAssetEditorToolkit>
	// however it should be safe, since commands are being used only within this editor
	// if it ever crashes, this function will have to go away and be reimplemented in each derived class

	GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::SelectAllNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanSelectAllNodes)
		);

	GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::DeleteSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanDeleteNodes)
		);

	GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::CopySelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanCopyNodes)
	);
	
	GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::CutSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanCutNodes)
	);

	GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::PasteNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanPasteNodes)
	);

	GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::DuplicateNodes),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanDuplicateNodes)
		);

	GraphEditorCommands->MapAction(
		FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateRaw(this, &FAIGraphEditor::OnCreateComment),
		FCanExecuteAction::CreateRaw(this, &FAIGraphEditor::CanCreateComment)
	);
}

FGraphPanelSelectionSet FAIGraphEditor::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = UpdateGraphEdPtr.Pin())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}

	return CurrentSelection;
}

void FAIGraphEditor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	// empty in base class
}

void FAIGraphEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		// Clear selection, to avoid holding refs to nodes that go away
		if (TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin())
		{
			CurrentGraphEditor->ClearSelectionSet();
			CurrentGraphEditor->NotifyGraphChanged();
		}
		FSlateApplication::Get().DismissAllMenus();
	}
}

void FAIGraphEditor::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		// Clear selection, to avoid holding refs to nodes that go away
		if (TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin())
		{
			CurrentGraphEditor->ClearSelectionSet();
			CurrentGraphEditor->NotifyGraphChanged();
		}
		FSlateApplication::Get().DismissAllMenus();
	}
}

void FAIGraphEditor::SelectAllNodes()
{
	if (TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin())
	{
		CurrentGraphEditor->SelectAllNodes();
	}
}

bool FAIGraphEditor::CanSelectAllNodes() const
{
	return true;
}

void FAIGraphEditor::DeleteSelectedNodes()
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (!CurrentGraphEditor.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(FGenericCommands::Get().Delete->GetDescription());
	CurrentGraphEditor->GetCurrentGraph()->Modify();

	const FGraphPanelSelectionSet SelectedNodes = CurrentGraphEditor->GetSelectedNodes();
	CurrentGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Node->CanUserDeleteNode())
			{
				Node->Modify();
				Node->DestroyNode();
			}
		}
	}
}

bool FAIGraphEditor::CanDeleteNodes() const
{
	// If any of the nodes can be deleted then we should allow deleting
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if (Node && Node->CanUserDeleteNode())
		{
			return true;
		}
	}

	return false;
}

void FAIGraphEditor::DeleteSelectedDuplicatableNodes()
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (!CurrentGraphEditor.IsValid())
	{
		return;
	}

	const FGraphPanelSelectionSet OldSelectedNodes = CurrentGraphEditor->GetSelectedNodes();
	CurrentGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if (Node && Node->CanDuplicateNode())
		{
			CurrentGraphEditor->SetNodeSelection(Node, true);
		}
	}

	// Delete the duplicatable nodes
	DeleteSelectedNodes();

	CurrentGraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			CurrentGraphEditor->SetNodeSelection(Node, true);
		}
	}
}

void FAIGraphEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedDuplicatableNodes();
}

bool FAIGraphEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FAIGraphEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	TArray<UAIGraphNode*> SubNodes;

	FString ExportedText;

	int32 CopySubNodeIndex = 0;
	for (FGraphPanelSelectionSet::TIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		UAIGraphNode* AINode = Cast<UAIGraphNode>(Node);
		if (Node == nullptr)
		{
			SelectedIter.RemoveCurrent();
			continue;
		}

		Node->PrepareForCopying();

		if (AINode)
		{
			AINode->CopySubNodeIndex = CopySubNodeIndex;

			// append all subnodes for selection
			for (int32 Idx = 0; Idx < AINode->SubNodes.Num(); Idx++)
			{
				AINode->SubNodes[Idx]->CopySubNodeIndex = CopySubNodeIndex;
				SubNodes.Add(AINode->SubNodes[Idx]);
			}

			CopySubNodeIndex++;
		}
	}

	for (int32 Idx = 0; Idx < SubNodes.Num(); Idx++)
	{
		SelectedNodes.Add(SubNodes[Idx]);
		SubNodes[Idx]->PrepareForCopying();
	}

	FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	for (FGraphPanelSelectionSet::TIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UAIGraphNode* Node = Cast<UAIGraphNode>(*SelectedIter);
		if (Node)
		{
			Node->PostCopyNode();
		}
	}
}

bool FAIGraphEditor::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if (Node && Node->CanDuplicateNode())
		{
			return true;
		}
	}

	return false;
}

void FAIGraphEditor::PasteNodes()
{
	if (TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin())
	{
		PasteNodesHere(CurrentGraphEditor->GetPasteLocation2f());
	}
}

void FAIGraphEditor::PasteNodesHere(const FVector2D& Location)
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (!CurrentGraphEditor.IsValid())
	{
		return;
	}

	// Undo/Redo support
	const FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());
	UEdGraph* EdGraph = CurrentGraphEditor->GetCurrentGraph();
	UAIGraph* AIGraph = Cast<UAIGraph>(EdGraph);

	EdGraph->Modify();
	if (AIGraph)
	{
		AIGraph->LockUpdates();
	}

	UAIGraphNode* SelectedParent = NULL;
	bool bHasMultipleNodesSelected = false;

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UAIGraphNode* Node = Cast<UAIGraphNode>(*SelectedIter);
		if (Node && Node->IsSubNode())
		{
			Node = Node->ParentNode;
		}

		if (Node)
		{
			if (SelectedParent == nullptr)
			{
				SelectedParent = Node;
			}
			else
			{
				bHasMultipleNodesSelected = true;
				break;
			}
		}
	}

	// Clear the selection set (newly pasted stuff will be selected)
	CurrentGraphEditor->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(EdGraph, TextToImport, /*out*/ PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f, 0.0f);
	
	// Number of nodes used to calculate AvgNodePosition
	int32 AvgCount = 0;

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* EdNode = *It;
		UAIGraphNode* AINode = Cast<UAIGraphNode>(EdNode);
		if (EdNode && (AINode == nullptr || !AINode->IsSubNode()))
		{
			AvgNodePosition.X += EdNode->NodePosX;
			AvgNodePosition.Y += EdNode->NodePosY;
			++AvgCount;
		}
	}

	if (AvgCount > 0)
	{
		float InvNumNodes = 1.0f / float(AvgCount);
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	bool bPastedParentNode = false;

	TMap<FGuid/*New*/, FGuid/*Old*/> NewToOldNodeMapping;

	TMap<int32, UAIGraphNode*> ParentMap;
	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* PasteNode = *It;
		UAIGraphNode* PasteAINode = Cast<UAIGraphNode>(PasteNode);

		if (PasteNode && (PasteAINode == nullptr || !PasteAINode->IsSubNode()))
		{
			bPastedParentNode = true;

			// Select the newly pasted stuff
			CurrentGraphEditor->SetNodeSelection(PasteNode, true);

			const FVector::FReal NodePosX = (PasteNode->NodePosX - AvgNodePosition.X) + Location.X;
			const FVector::FReal NodePosY = (PasteNode->NodePosY - AvgNodePosition.Y) + Location.Y;

			PasteNode->NodePosX = static_cast<int32>(NodePosX);
			PasteNode->NodePosY = static_cast<int32>(NodePosY);

			PasteNode->SnapToGrid(16);

			const FGuid OldGuid = PasteNode->NodeGuid;

			// Give new node a different Guid from the old one
			PasteNode->CreateNewGuid();

			const FGuid NewGuid = PasteNode->NodeGuid;

			NewToOldNodeMapping.Add(NewGuid, OldGuid);

			if (PasteAINode)
			{
				PasteAINode->RemoveAllSubNodes();
				ParentMap.Add(PasteAINode->CopySubNodeIndex, PasteAINode);
			}
		}
	}

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UAIGraphNode* PasteNode = Cast<UAIGraphNode>(*It);
		if (PasteNode && PasteNode->IsSubNode())
		{
			const FGuid OldGuid = PasteNode->NodeGuid;

			PasteNode->NodePosX = 0;
			PasteNode->NodePosY = 0;

			// remove subnode from graph, it will be referenced from parent node
			PasteNode->DestroyNode();

			PasteNode->ParentNode = ParentMap.FindRef(PasteNode->CopySubNodeIndex);
			if (PasteNode->ParentNode)
			{
				PasteNode->ParentNode->AddSubNode(PasteNode, EdGraph);
			}
			else if (!bHasMultipleNodesSelected && !bPastedParentNode && SelectedParent)
			{
				PasteNode->ParentNode = SelectedParent;
				SelectedParent->AddSubNode(PasteNode, EdGraph);
			}

			// Adding the sub-node will assign it a new GUID
			const FGuid NewGuid = PasteNode->NodeGuid;
			NewToOldNodeMapping.Add(NewGuid, OldGuid);
		}
	}

	FixupPastedNodes(PastedNodes, NewToOldNodeMapping);

	if (AIGraph)
	{
		AIGraph->UpdateClassData();
		AIGraph->OnNodesPasted(TextToImport);
		AIGraph->UnlockUpdates();
	}

	// Update UI
	CurrentGraphEditor->NotifyGraphChanged();

	UObject* GraphOwner = EdGraph->GetOuter();
	if (GraphOwner)
	{
		GraphOwner->PostEditChange();
		GraphOwner->MarkPackageDirty();
	}
}

void FAIGraphEditor::FixupPastedNodes(const TSet<UEdGraphNode*>& PastedGraphNodes, const TMap<FGuid/*New*/, FGuid/*Old*/>& NewToOldNodeMapping)
{
	
}

bool FAIGraphEditor::CanPasteNodes() const
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (!CurrentGraphEditor.IsValid())
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(CurrentGraphEditor->GetCurrentGraph(), ClipboardContent);
}

void FAIGraphEditor::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool FAIGraphEditor::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

bool FAIGraphEditor::CanCreateComment() const
{
 	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	return CurrentGraphEditor.IsValid();
}

void FAIGraphEditor::OnCreateComment()
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (UEdGraph* EdGraph = CurrentGraphEditor.IsValid() ? CurrentGraphEditor->GetCurrentGraph() : nullptr)
	{
		TSharedPtr<FEdGraphSchemaAction> Action = EdGraph->GetSchema()->GetCreateCommentAction();
		if (Action.IsValid())
		{
			Action->PerformAction(EdGraph, nullptr, FVector2f());
		}
	}
}

void FAIGraphEditor::OnClassListUpdated()
{
	TSharedPtr<SGraphEditor> CurrentGraphEditor = UpdateGraphEdPtr.Pin();
	if (!CurrentGraphEditor.IsValid())
	{
		return;
	}

	UAIGraph* MyGraph = Cast<UAIGraph>(CurrentGraphEditor->GetCurrentGraph());
	if (MyGraph)
	{
		const bool bUpdated = MyGraph->UpdateUnknownNodeClasses();
		if (bUpdated)
		{
			FGraphPanelSelectionSet CurrentSelection = GetSelectedNodes();
			OnSelectedNodesChanged(CurrentSelection);

			MyGraph->UpdateAsset();
		}
	}
}


#undef LOCTEXT_NAMESPACE
