// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraphNode.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"
#include "AIGraph.h"
#include "DiffResults.h"
#include "ScopedTransaction.h"
#include "BlueprintNodeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIGraphNode)

#define LOCTEXT_NAMESPACE "AIGraph"

UAIGraphNode::UAIGraphNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeInstance = nullptr;
	CopySubNodeIndex = 0;
	bIsReadOnly = false;
	bIsSubNode = false;
}

void UAIGraphNode::InitializeInstance()
{
	// empty in base class
}

void UAIGraphNode::PostPlacedNewNode()
{
	// NodeInstance can be already spawned by paste operation, don't override it

	UClass* NodeClass = ClassData.GetClass(true);
	if (NodeClass && (NodeInstance == nullptr))
	{
		UEdGraph* MyGraph = GetGraph();
		UObject* GraphOwner = MyGraph ? MyGraph->GetOuter() : nullptr;
		if (GraphOwner)
		{
			NodeInstance = NewObject<UObject>(GraphOwner, NodeClass);
			NodeInstance->SetFlags(RF_Transactional);
			InitializeInstance();
			UpdateErrorMessage();
		}
	}
}

bool UAIGraphNode::CanDuplicateNode() const
{
	return bIsReadOnly ? false : Super::CanDuplicateNode();
}

bool UAIGraphNode::CanUserDeleteNode() const
{
	return bIsReadOnly ? false : Super::CanUserDeleteNode();
}

void UAIGraphNode::PrepareForCopying()
{
	if (NodeInstance)
	{
		// Temporarily take ownership of the node instance, so that it is not deleted when cutting
		NodeInstance->Rename(nullptr, this, REN_DontCreateRedirectors | REN_DoNotDirty);
	}
}
#if WITH_EDITOR

void UAIGraphNode::PostEditImport()
{
	ResetNodeOwner();

	if (NodeInstance)
	{
		InitializeInstance();
	}
}

void UAIGraphNode::PostEditUndo()
{
	UEdGraphNode::PostEditUndo();
	ResetNodeOwner();
	
	if (ParentNode)
	{
		ParentNode->SubNodes.AddUnique(this);
	}
}

#endif

void UAIGraphNode::PostCopyNode()
{
	ResetNodeOwner();
}

void UAIGraphNode::ResetNodeOwner()
{
	if (NodeInstance)
	{
		UEdGraph* MyGraph = GetGraph();
		UObject* GraphOwner = MyGraph ? MyGraph->GetOuter() : nullptr;

		NodeInstance->Rename(NULL, GraphOwner, REN_DontCreateRedirectors | REN_DoNotDirty);
		NodeInstance->ClearFlags(RF_Transient);

		for (auto& SubNode : SubNodes)
		{
			SubNode->ResetNodeOwner();
		}
	}
}

FText UAIGraphNode::GetDescription() const
{
	FString StoredClassName = ClassData.GetClassName();
	StoredClassName.RemoveFromEnd(TEXT("_C"));
	
	return FText::Format(LOCTEXT("NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
}

FText UAIGraphNode::GetTooltipText() const
{
	FText TooltipDesc;

	if (!NodeInstance)
	{
		FString StoredClassName = ClassData.GetClassName();
		StoredClassName.RemoveFromEnd(TEXT("_C"));

		TooltipDesc = FText::Format(LOCTEXT("NodeClassError", "Class {0} not found, make sure it's saved!"), FText::FromString(StoredClassName));
	}
	else
	{
		if (ErrorMessage.Len() > 0)
		{
			TooltipDesc = FText::FromString(ErrorMessage);
		}
		else
		{
			if (NodeInstance->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) && !NodeInstance->GetClass()->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
			{
				FAssetData AssetData(NodeInstance->GetClass()->ClassGeneratedBy);
				FString Description = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDescription));
				if (!Description.IsEmpty())
				{
					Description.ReplaceInline(TEXT("\\n"), TEXT("\n"));
					TooltipDesc = FText::FromString(MoveTemp(Description));
				}
			}
			else
			{
				TooltipDesc = NodeInstance->GetClass()->GetToolTipText();
			}
		}
	}

	return TooltipDesc;
}

UEdGraphPin* UAIGraphNode::GetInputPin(int32 InputIndex) const
{
	check(InputIndex >= 0);

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return nullptr;
}

UEdGraphPin* UAIGraphNode::GetOutputPin(int32 InputIndex) const
{
	check(InputIndex >= 0);

	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Output)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return nullptr;
}

void UAIGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (FromPin != nullptr)
	{
		UEdGraphPin* OutputPin = GetOutputPin();

		if (GetSchema()->TryCreateConnection(FromPin, GetInputPin()))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
		else if (OutputPin != nullptr && GetSchema()->TryCreateConnection(OutputPin, FromPin))
		{
			NodeConnectionListChanged();
		}
	}
}

UAIGraph* UAIGraphNode::GetAIGraph()
{
	return CastChecked<UAIGraph>(GetGraph());
}

bool UAIGraphNode::IsSubNode() const
{
	return bIsSubNode || (ParentNode != nullptr);
}

void UAIGraphNode::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	GetAIGraph()->UpdateAsset();
}

bool UAIGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	// override in child class
	return false;
}

FString UAIGraphNode::GetPropertyNameAndValueForDiff(const FProperty* Prop, const uint8* PropertyAddr) const
{
	return BlueprintNodeHelpers::DescribeProperty(Prop, PropertyAddr);
}

void UAIGraphNode::FindDiffs(UEdGraphNode* OtherNode, FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	if (UAIGraphNode* OtherGraphNode = Cast<UAIGraphNode>(OtherNode))
	{
		if (NodeInstance && OtherGraphNode->NodeInstance)
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.ToolTip = LOCTEXT("DIF_NodeInstancePropertyToolTip", "A property of the node instance has changed");
			Diff.Category = EDiffType::MODIFICATION;

			DiffProperties(NodeInstance->GetClass(), OtherGraphNode->NodeInstance->GetClass(), NodeInstance, OtherGraphNode->NodeInstance, Results, Diff);
		}
	}
}

void UAIGraphNode::AddSubNode(UAIGraphNode* SubNode, class UEdGraph* ParentGraph)
{
	const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
	ParentGraph->Modify();
	Modify();

	SubNode->SetFlags(RF_Transactional);

	// set outer to be the graph so it doesn't go away
	SubNode->Rename(nullptr, ParentGraph, REN_NonTransactional);
	SubNode->ParentNode = this;

	SubNode->CreateNewGuid();
	SubNode->PostPlacedNewNode();
	SubNode->AllocateDefaultPins();
	SubNode->AutowireNewNode(nullptr);

	SubNode->NodePosX = 0;
	SubNode->NodePosY = 0;

	SubNodes.Add(SubNode);
	OnSubNodeAdded(SubNode);

	ParentGraph->NotifyGraphChanged();
	GetAIGraph()->UpdateAsset();
}

void UAIGraphNode::OnSubNodeAdded(UAIGraphNode* SubNode)
{
	// empty in base class
}

void UAIGraphNode::RemoveSubNode(UAIGraphNode* SubNode)
{
	Modify();
	SubNodes.RemoveSingle(SubNode);

	OnSubNodeRemoved(SubNode);
}

void UAIGraphNode::RemoveAllSubNodes()
{
	SubNodes.Reset();
}

void UAIGraphNode::OnSubNodeRemoved(UAIGraphNode* SubNode)
{
	// empty in base class
}

int32 UAIGraphNode::FindSubNodeDropIndex(UAIGraphNode* SubNode) const
{
	const int32 InsertIndex = SubNodes.IndexOfByKey(SubNode);
	return InsertIndex;
}

void UAIGraphNode::InsertSubNodeAt(UAIGraphNode* SubNode, int32 DropIndex)
{
	if (DropIndex > -1)
	{
		SubNodes.Insert(SubNode, DropIndex);
	}
	else
	{
		SubNodes.Add(SubNode);
	}
}

void UAIGraphNode::DestroyNode()
{
	if (ParentNode)
	{
		ParentNode->RemoveSubNode(this);
	}

	UEdGraphNode::DestroyNode();
}

bool UAIGraphNode::UsesBlueprint() const
{
	return NodeInstance && NodeInstance->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
}

bool UAIGraphNode::RefreshNodeClass()
{
	bool bUpdated = false;
	if (NodeInstance == nullptr)
	{
		if (FGraphNodeClassHelper::IsClassKnown(ClassData))
		{
			PostPlacedNewNode();
			bUpdated = (NodeInstance != nullptr);
		}
		else
		{
			FGraphNodeClassHelper::AddUnknownClass(ClassData);
		}
	}

	return bUpdated;
}

void UAIGraphNode::UpdateNodeClassData()
{
	if (NodeInstance)
	{
		UpdateNodeClassDataFrom(NodeInstance->GetClass(), ClassData);
		UpdateErrorMessage();
	}
}

void UAIGraphNode::UpdateErrorMessage()
{
	ErrorMessage = ClassData.GetDeprecatedMessage();
}

void UAIGraphNode::UpdateNodeClassDataFrom(UClass* InstanceClass, FGraphNodeClassData& UpdatedData)
{
	if (InstanceClass)
	{
		if (UBlueprint* BPOwner = Cast<UBlueprint>(InstanceClass->ClassGeneratedBy))
		{
			UpdatedData = FGraphNodeClassData(BPOwner->GetName(), BPOwner->GetOutermost()->GetName(), InstanceClass->GetName(), InstanceClass);
		}
		else if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(InstanceClass))
		{
			UpdatedData = FGraphNodeClassData(BPGC->GetClassPathName(), BPGC);
		}
		else
		{
			UpdatedData = FGraphNodeClassData(InstanceClass, FGraphNodeClassHelper::GetDeprecationMessage(InstanceClass));
		}
	}
}

bool UAIGraphNode::HasErrors() const
{
	return ErrorMessage.Len() > 0 || NodeInstance == nullptr;
}

#undef LOCTEXT_NAMESPACE
