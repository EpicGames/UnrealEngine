// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraphSchema.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "Settings/EditorStyleSettings.h"
#include "EdGraph/EdGraph.h"
#include "AIGraphNode.h"
#include "GraphEditorActions.h"
#include "AIGraphConnectionDrawingPolicy.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "EdGraphNode_Comment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIGraphSchema)

#define LOCTEXT_NAMESPACE "AIGraph"

namespace
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	const int32 NodeDistance = 60;
}

//////////////////////////////////////////////////////////////////////////

UEdGraphNode* FAISchemaAction_AddComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode_Comment* const CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;

	TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(ParentGraph);
	if (GraphEditorPtr.IsValid())
	{
		// If they have a selection, build a bounding box around the selection
		if (GraphEditorPtr->GetBoundsForSelectedNodes(/*out*/ Bounds, 50.0f))
		{
			CommentTemplate->SetBounds(Bounds);
			SpawnLocation.X = CommentTemplate->NodePosX;
			SpawnLocation.Y = CommentTemplate->NodePosY;
		}
		else
		{
			// Otherwise initialize a default comment at the user's cursor location.
			SpawnLocation = GraphEditorPtr->GetPasteLocation2f();
		}
	}

	UEdGraphNode* const NewNode = FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);

	return NewNode;
}

//////////////////////////////////////////////////////////////////////////

UEdGraphNode* FAISchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode* ResultNode = NULL;

	// If there is a template, we actually use it
	if (NodeTemplate != NULL)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		NodeTemplate->SetFlags(RF_Transactional);

		// set outer to be the graph so it doesn't go away
		NodeTemplate->Rename(NULL, ParentGraph, REN_NonTransactional);
		ParentGraph->AddNode(NodeTemplate, true);

		NodeTemplate->CreateNewGuid();
		NodeTemplate->PostPlacedNewNode();

		// For input pins, new node will generally overlap node being dragged off
		// Work out if we want to visually push away from connected node
		int32 XLocation = static_cast<int32>(Location.X);
		if (FromPin && FromPin->Direction == EGPD_Input)
		{
			UEdGraphNode* PinNode = FromPin->GetOwningNode();
			const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

			if (XDelta < NodeDistance)
			{
				// Set location to edge of current node minus the max move distance
				// to force node to push off from connect node enough to give selection handle
				XLocation = PinNode->NodePosX - NodeDistance;
			}
		}

		NodeTemplate->NodePosX = XLocation;
		NodeTemplate->NodePosY = static_cast<int32>(Location.Y);
		NodeTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

		// setup pins after placing node in correct spot, since pin sorting will happen as soon as link connection change occurs
		NodeTemplate->AllocateDefaultPins();
		NodeTemplate->AutowireNewNode(FromPin);

		ResultNode = NodeTemplate;
	}

	return ResultNode;
}

UEdGraphNode* FAISchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode* ResultNode = NULL;
	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location);

		// Try autowiring the rest of the pins
		for (int32 Index = 1; Index < FromPins.Num(); ++Index)
		{
			ResultNode->AutowireNewNode(FromPins[Index]);
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, NULL, Location, bSelectNewNode);
	}

	return ResultNode;
}

void FAISchemaAction_NewNode::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdGraphSchemaAction::AddReferencedObjects(Collector);

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject(NodeTemplate);
}

UEdGraphNode* FAISchemaAction_NewSubNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	ParentNode->AddSubNode(NodeTemplate, ParentGraph);
	return NULL;
}

UEdGraphNode* FAISchemaAction_NewSubNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	return PerformAction(ParentGraph, NULL, Location, bSelectNewNode);
}

void FAISchemaAction_NewSubNode::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdGraphSchemaAction::AddReferencedObjects(Collector);

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject(NodeTemplate);
	Collector.AddReferencedObject(ParentNode);
}

//////////////////////////////////////////////////////////////////////////

UAIGraphSchema::UAIGraphSchema(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

TSharedPtr<FAISchemaAction_NewNode> UAIGraphSchema::AddNewNodeAction(FGraphActionListBuilderBase& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip)
{
	TSharedPtr<FAISchemaAction_NewNode> NewAction = TSharedPtr<FAISchemaAction_NewNode>(new FAISchemaAction_NewNode(Category, MenuDesc, Tooltip, 0));
	ContextMenuBuilder.AddAction(NewAction);

	return NewAction;
}

TSharedPtr<FAISchemaAction_NewSubNode> UAIGraphSchema::AddNewSubNodeAction(FGraphActionListBuilderBase& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip)
{
	TSharedPtr<FAISchemaAction_NewSubNode> NewAction = TSharedPtr<FAISchemaAction_NewSubNode>(new FAISchemaAction_NewSubNode(Category, MenuDesc, Tooltip, 0));
	ContextMenuBuilder.AddAction(NewAction);
	return NewAction;
}

void UAIGraphSchema::GetSubNodeClasses(int32 SubNodeFlags, TArray<FGraphNodeClassData>& ClassData, UClass*& GraphNodeClass) const
{
	// empty in base class
}

void UAIGraphSchema::GetGraphNodeContextActions(FGraphContextMenuBuilder& ContextMenuBuilder, int32 SubNodeFlags) const
{
	UEdGraph* Graph = (UEdGraph*)ContextMenuBuilder.CurrentGraph;
	UClass* GraphNodeClass = nullptr;
	TArray<FGraphNodeClassData> NodeClasses;
	GetSubNodeClasses(SubNodeFlags, NodeClasses, GraphNodeClass);

	if (GraphNodeClass)
	{
		for (const auto& NodeClass : NodeClasses)
		{
			const FText NodeTypeName = FText::FromString(FName::NameToDisplayString(NodeClass.ToString(), false));

			UAIGraphNode* OpNode = NewObject<UAIGraphNode>(Graph, GraphNodeClass);
			OpNode->ClassData = NodeClass;

			TSharedPtr<FAISchemaAction_NewSubNode> AddOpAction = UAIGraphSchema::AddNewSubNodeAction(ContextMenuBuilder, NodeClass.GetCategory(), NodeTypeName, NodeClass.GetTooltip());
			AddOpAction->ParentNode = Cast<UAIGraphNode>(ContextMenuBuilder.SelectedObjects[0]);
			AddOpAction->NodeTemplate = OpNode;
		}
	}
}

void UAIGraphSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("BehaviorTreeGraphSchemaNodeActions", LOCTEXT("ClassActionsMenuHeader", "Node Actions"));
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
			Section.AddMenuEntry(FGenericCommands::Get().Cut);
			Section.AddMenuEntry(FGenericCommands::Get().Copy);
			Section.AddMenuEntry(FGenericCommands::Get().Duplicate);

			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		}
	}

	Super::GetContextMenuActions(Menu, Context);
}

void UAIGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakNodeLinks", "Break Node Links"));

	Super::BreakNodeLinks(TargetNode);
}

void UAIGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links"));

	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);
}

void UAIGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link"));

	Super::BreakSinglePinLink(SourcePin, TargetPin);
}

FLinearColor UAIGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FColor::White;
}

bool UAIGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	check(Pin != NULL);
	return Pin->bDefaultValueIsIgnored;
}

class FConnectionDrawingPolicy* UAIGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FAIGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

TSharedPtr<FEdGraphSchemaAction> UAIGraphSchema::GetCreateCommentAction() const
{
	return TSharedPtr<FEdGraphSchemaAction>(static_cast<FEdGraphSchemaAction*>(new FAISchemaAction_AddComment));
}

int32 UAIGraphSchema::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	if (Graph)
	{
		TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(Graph);
		if (GraphEditorPtr.IsValid())
		{
			return GraphEditorPtr->GetNumberOfSelectedNodes();
		}
	}

	return 0;
}

#undef LOCTEXT_NAMESPACE
