// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIGraphTypes.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Blueprint.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "ObjectEditorUtils.h"
#include "Logging/MessageLog.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIGraphTypes)

#define LOCTEXT_NAMESPACE "AIGraph"

FGraphNodeClassData::FGraphNodeClassData(UClass* InClass, const FString& InDeprecatedMessage) :
	bIsHidden(0),
	bHideParent(0),
	Class(InClass),
	DeprecatedMessage(InDeprecatedMessage)
{
	Category = GetCategory();

	if (InClass)
	{
		ClassName = InClass->GetName();
	}
}

FGraphNodeClassData::FGraphNodeClassData(const FTopLevelAssetPath& InGeneratedClassPath, UClass* InClass) :
	bIsHidden(0),
	bHideParent(0),
	Class(InClass),
	AssetName(InGeneratedClassPath.GetAssetName().ToString()),
	GeneratedClassPackage(InGeneratedClassPath.GetPackageName().ToString()),
	ClassName(InGeneratedClassPath.GetAssetName().ToString())
{
	Category = GetCategory();
}

FGraphNodeClassData::FGraphNodeClassData(const FString& InAssetName, const FString& InGeneratedClassPackage, const FString& InClassName, UClass* InClass) :
	bIsHidden(0),
	bHideParent(0),
	Class(InClass),
	AssetName(InAssetName),
	GeneratedClassPackage(InGeneratedClassPackage),
	ClassName(InClassName)
{
	Category = GetCategory();
}

FString FGraphNodeClassData::ToString() const
{
	FString ShortName = GetDisplayName();
	if (!ShortName.IsEmpty())
	{
		return ShortName;
	}

	UClass* MyClass = Class.Get();
	if (MyClass)
	{
		FString ClassDesc = MyClass->GetName();

		if (MyClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			return ClassDesc.LeftChop(2);
		}

		const int32 ShortNameIdx = ClassDesc.Find(TEXT("_"), ESearchCase::CaseSensitive);
		if (ShortNameIdx != INDEX_NONE)
		{
			ClassDesc.MidInline(ShortNameIdx + 1, MAX_int32, EAllowShrinking::No);
		}

		return ClassDesc;
	}

	return AssetName;
}

FString FGraphNodeClassData::GetClassName() const
{
	return Class.IsValid() ? Class->GetName() : ClassName;
}

FString FGraphNodeClassData::GetDisplayName() const
{
	return Class.IsValid() ? Class->GetMetaData(TEXT("DisplayName")) : FString();
}

FText FGraphNodeClassData::GetTooltip() const
{
	return Class.IsValid() ? Class->GetToolTipText() : FText::GetEmpty();
}

FText FGraphNodeClassData::GetCategory() const
{
	return Class.IsValid() ? FObjectEditorUtils::GetCategoryText(Class.Get()) : Category;
}

bool FGraphNodeClassData::IsAbstract() const
{
	return Class.IsValid() ? Class.Get()->HasAnyClassFlags(CLASS_Abstract) : false;
}

UClass* FGraphNodeClassData::GetClass(bool bSilent)
{
	UClass* RetClass = Class.Get();
	if (RetClass == NULL && GeneratedClassPackage.Len())
	{
		GWarn->BeginSlowTask(LOCTEXT("LoadPackage", "Loading Package..."), true);

		UPackage* Package = LoadPackage(NULL, *GeneratedClassPackage, LOAD_NoRedirects);
		if (Package)
		{
			Package->FullyLoad();

			RetClass = FindObject<UClass>(Package, *ClassName);

			GWarn->EndSlowTask();
			Class = RetClass;
		}
		else
		{
			GWarn->EndSlowTask();

			if (!bSilent)
			{
				FMessageLog EditorErrors("EditorErrors");
				EditorErrors.Error(LOCTEXT("PackageLoadFail", "Package Load Failed"));
				EditorErrors.Info(FText::FromString(GeneratedClassPackage));
				EditorErrors.Notify(LOCTEXT("PackageLoadFail", "Package Load Failed"));
			}
		}
	}

	return RetClass;
}

//////////////////////////////////////////////////////////////////////////
TArray<FName> FGraphNodeClassHelper::UnknownPackages;
TMap<UClass*, int32> FGraphNodeClassHelper::BlueprintClassCount;
FGraphNodeClassHelper::FOnPackageListUpdated FGraphNodeClassHelper::OnPackageListUpdated;

FGraphNodeClassHelper::FGraphNodeClassHelper(UClass* InRootClass)
{
	RootNodeClass = InRootClass;

	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().AddRaw(this, &FGraphNodeClassHelper::InvalidateCache);
	AssetRegistryModule.Get().OnAssetAdded().AddRaw(this, &FGraphNodeClassHelper::OnAssetAdded);
	AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FGraphNodeClassHelper::OnAssetRemoved);

	// Register to have Populate called when doing a Reload.
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FGraphNodeClassHelper::OnReloadComplete);

	// Register to have Populate called when a Blueprint is compiled.
	GEditor->OnBlueprintCompiled().AddRaw(this, &FGraphNodeClassHelper::InvalidateCache);
	GEditor->OnClassPackageLoadedOrUnloaded().AddRaw(this, &FGraphNodeClassHelper::InvalidateCache);

	UpdateAvailableBlueprintClasses();
}

FGraphNodeClassHelper::~FGraphNodeClassHelper()
{
	// Unregister with the Asset Registry to be informed when it is done loading up files.
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}

		// Unregister to have Populate called when doing a Reload.
		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

		// Unregister to have Populate called when a Blueprint is compiled.
		if (UObjectInitialized())
		{
			// GEditor can't have been destructed before we call this or we'll crash.
			GEditor->OnBlueprintCompiled().RemoveAll(this);
			GEditor->OnClassPackageLoadedOrUnloaded().RemoveAll(this);
		}
	}
}

void FGraphNodeClassNode::AddUniqueSubNode(TSharedPtr<FGraphNodeClassNode> SubNode)
{
	for (int32 Idx = 0; Idx < SubNodes.Num(); Idx++)
	{
		if (SubNode->Data.GetClassName() == SubNodes[Idx]->Data.GetClassName())
		{
			return;
		}
	}

	SubNodes.Add(SubNode);
}

void FGraphNodeClassHelper::GatherClasses(const UClass* BaseClass, TArray<FGraphNodeClassData>& AvailableClasses)
{
	const FString BaseClassName = BaseClass->GetName();
	if (!RootNode.IsValid())
	{
		BuildClassGraph();
	}

	TSharedPtr<FGraphNodeClassNode> BaseNode = FindBaseClassNode(RootNode, BaseClassName);
	FindAllSubClasses(BaseNode, AvailableClasses);
}

FString FGraphNodeClassHelper::GetDeprecationMessage(const UClass* Class)
{
	static FName MetaDeprecated = TEXT("DeprecatedNode");
	static FName MetaDeprecatedMessage = TEXT("DeprecationMessage");
	FString DefDeprecatedMessage("Please remove it!");
	FString DeprecatedPrefix("DEPRECATED");
	FString DeprecatedMessage;

	if (Class && Class->HasAnyClassFlags(CLASS_Native) && Class->HasMetaData(MetaDeprecated))
	{
		DeprecatedMessage = DeprecatedPrefix + TEXT(": ");
		DeprecatedMessage += Class->HasMetaData(MetaDeprecatedMessage) ? Class->GetMetaData(MetaDeprecatedMessage) : DefDeprecatedMessage;
	}

	return DeprecatedMessage;
}

bool FGraphNodeClassHelper::IsClassKnown(const FGraphNodeClassData& ClassData)
{
	return !ClassData.IsBlueprint() || !UnknownPackages.Contains(*ClassData.GetPackageName());
}

void FGraphNodeClassHelper::AddUnknownClass(const FGraphNodeClassData& ClassData)
{
	if (ClassData.IsBlueprint())
	{
		UnknownPackages.AddUnique(*ClassData.GetPackageName());
	}
}

bool FGraphNodeClassHelper::IsHidingParentClass(UClass* Class)
{
	static FName MetaHideParent = TEXT("HideParentNode");
	return Class && Class->HasAnyClassFlags(CLASS_Native) && Class->HasMetaData(MetaHideParent);
}

bool FGraphNodeClassHelper::IsHidingClass(UClass* Class)
{
	static FName MetaHideInEditor = TEXT("HiddenNode");

	return 
		Class && 
		((Class->HasAnyClassFlags(CLASS_Native) && Class->HasMetaData(MetaHideInEditor))
		|| ForcedHiddenClasses.Contains(Class));
}

bool FGraphNodeClassHelper::IsPackageSaved(FName PackageName)
{
	const bool bFound = FPackageName::SearchForPackageOnDisk(PackageName.ToString());
	return bFound;
}

void FGraphNodeClassHelper::OnAssetAdded(const struct FAssetData& AssetData)
{
	if (AssetData.IsInstanceOf<UBlueprint>())
	{
		TSharedPtr<FGraphNodeClassNode> Node = CreateClassDataNode(AssetData);

		TSharedPtr<FGraphNodeClassNode> ParentNode;
		if (Node.IsValid())
		{
			ParentNode = FindBaseClassNode(RootNode, Node->ParentClassName);

			if (!IsPackageSaved(AssetData.PackageName))
			{
				UnknownPackages.AddUnique(AssetData.PackageName);
			}
			else
			{
				const int32 PrevListCount = UnknownPackages.Num();
				UnknownPackages.RemoveSingleSwap(AssetData.PackageName);

				if (UnknownPackages.Num() != PrevListCount)
				{
					OnPackageListUpdated.Broadcast();
				}
			}
		}

		if (ParentNode.IsValid())
		{
			ParentNode->AddUniqueSubNode(Node);
			Node->ParentNode = ParentNode;
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (!AssetRegistryModule.Get().IsLoadingAssets())
		{
			UpdateAvailableBlueprintClasses();
		}
	}
}

void FGraphNodeClassHelper::OnAssetRemoved(const struct FAssetData& AssetData)
{
	FString AssetClassName;
	if (AssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, AssetClassName))
	{
		ConstructorHelpers::StripObjectClass(AssetClassName);
		AssetClassName = FPackageName::ObjectPathToObjectName(AssetClassName);

		TSharedPtr<FGraphNodeClassNode> Node = FindBaseClassNode(RootNode, AssetClassName);
		if (Node.IsValid() && Node->ParentNode.IsValid())
		{
			Node->ParentNode->SubNodes.RemoveSingleSwap(Node);
		}
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (!AssetRegistryModule.Get().IsLoadingAssets())
	{
		UpdateAvailableBlueprintClasses();
	}
}

void FGraphNodeClassHelper::InvalidateCache()
{
	RootNode.Reset();

	UpdateAvailableBlueprintClasses();
}

void FGraphNodeClassHelper::OnReloadComplete(EReloadCompleteReason Reason)
{
	InvalidateCache();
}

TSharedPtr<FGraphNodeClassNode> FGraphNodeClassHelper::CreateClassDataNode(const struct FAssetData& AssetData)
{
	TSharedPtr<FGraphNodeClassNode> Node;

	FString AssetParentClassName;
	if (AssetData.GetTagValue(FBlueprintTags::ParentClassPath, AssetParentClassName))
	{
		UObject* Outer2(NULL);
		ResolveName(Outer2, AssetParentClassName, false, false);

		Node = MakeShareable(new FGraphNodeClassNode);
		Node->ParentClassName = AssetParentClassName;

		FGraphNodeClassData NewData(AssetData.AssetName.ToString(), AssetData.PackageName.ToString(), AssetData.AssetName.ToString() + TEXT("_C"), nullptr);
		Node->Data = NewData;
	}

	return Node;
}

TSharedPtr<FGraphNodeClassNode> FGraphNodeClassHelper::FindBaseClassNode(TSharedPtr<FGraphNodeClassNode> Node, const FString& ClassName)
{
	TSharedPtr<FGraphNodeClassNode> RetNode;
	if (Node.IsValid())
	{
		if (Node->Data.GetClassName() == ClassName)
		{
			return Node;
		}

		for (int32 i = 0; i < Node->SubNodes.Num(); i++)
		{
			RetNode = FindBaseClassNode(Node->SubNodes[i], ClassName);
			if (RetNode.IsValid())
			{
				break;
			}
		}
	}

	return RetNode;
}

void FGraphNodeClassHelper::FindAllSubClasses(TSharedPtr<FGraphNodeClassNode> Node, TArray<FGraphNodeClassData>& AvailableClasses)
{
	if (Node.IsValid())
	{
		if (!Node->Data.IsAbstract() && !Node->Data.IsDeprecated() && !Node->Data.bIsHidden)
		{
			AvailableClasses.Add(Node->Data);
		}

		for (int32 i = 0; i < Node->SubNodes.Num(); i++)
		{
			FindAllSubClasses(Node->SubNodes[i], AvailableClasses);
		}
	}
}

void FGraphNodeClassHelper::BuildClassGraph()
{
	TArray<TSharedPtr<FGraphNodeClassNode> > NodeList;
	TArray<UClass*> HideParentList;
	RootNode.Reset();

	// gather all native classes
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* TestClass = *It;
		if (TestClass->HasAnyClassFlags(CLASS_Native) && TestClass->IsChildOf(RootNodeClass))
		{
			TSharedPtr<FGraphNodeClassNode> NewNode = MakeShareable(new FGraphNodeClassNode);
			NewNode->ParentClassName = TestClass->GetSuperClass()->GetName();

			FString DeprecatedMessage = GetDeprecationMessage(TestClass);
			FGraphNodeClassData NewData(TestClass, DeprecatedMessage);

			NewData.bHideParent = IsHidingParentClass(TestClass);
			if (NewData.bHideParent)
			{
				HideParentList.Add(TestClass->GetSuperClass());
			}

			NewData.bIsHidden = IsHidingClass(TestClass);

			NewNode->Data = NewData;

			if (TestClass == RootNodeClass)
			{
				RootNode = NewNode;
			}

			NodeList.Add(NewNode);
		}
	}

	// find all hidden parent classes
	for (int32 i = 0; i < NodeList.Num(); i++)
	{
		TSharedPtr<FGraphNodeClassNode> TestNode = NodeList[i];
		if (HideParentList.Contains(TestNode->Data.GetClass()))
		{
			TestNode->Data.bIsHidden = true;
		}
	}

	// gather all blueprints
	if (bGatherBlueprints)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FAssetData> BlueprintList;

		FARFilter Filter;
		Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
		AssetRegistryModule.Get().GetAssets(Filter, BlueprintList, false);

		for (int32 i = 0; i < BlueprintList.Num(); i++)
		{
			TSharedPtr<FGraphNodeClassNode> NewNode = CreateClassDataNode(BlueprintList[i]);
			NodeList.Add(NewNode);
		}
	}

	// build class tree
	AddClassGraphChildren(RootNode, NodeList);
}

void FGraphNodeClassHelper::AddClassGraphChildren(TSharedPtr<FGraphNodeClassNode> Node, TArray<TSharedPtr<FGraphNodeClassNode> >& NodeList)
{
	if (!Node.IsValid())
	{
		return;
	}

	const FString NodeClassName = Node->Data.GetClassName();
	for (int32 i = NodeList.Num() - 1; i >= 0; i--)
	{
		if (NodeList[i]->ParentClassName == NodeClassName)
		{
			TSharedPtr<FGraphNodeClassNode> MatchingNode = NodeList[i];
			NodeList.RemoveAt(i);

			MatchingNode->ParentNode = Node;
			Node->SubNodes.Add(MatchingNode);

			AddClassGraphChildren(MatchingNode, NodeList);
		}
	}
}

int32 FGraphNodeClassHelper::GetObservedBlueprintClassCount(UClass* BaseNativeClass)
{
	return BlueprintClassCount.FindRef(BaseNativeClass);
}

void FGraphNodeClassHelper::AddObservedBlueprintClasses(UClass* BaseNativeClass)
{
	BlueprintClassCount.Add(BaseNativeClass, 0);
}

void FGraphNodeClassHelper::UpdateAvailableBlueprintClasses()
{
	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		const bool bSearchSubClasses = true;

		TArray<FTopLevelAssetPath> ClassNames;
		TSet<FTopLevelAssetPath> DerivedClassNames;

		for (TMap<UClass*, int32>::TIterator It(BlueprintClassCount); It; ++It)
		{
			ClassNames.Reset();
			ClassNames.Add(It.Key()->GetClassPathName());

			DerivedClassNames.Empty(DerivedClassNames.Num());
			AssetRegistry.GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

			int32& Count = It.Value();
			Count = DerivedClassNames.Num();
		}
	}
}

void FGraphNodeClassHelper::AddForcedHiddenClass(UClass* Class)
{
	if (Class)
	{
		ForcedHiddenClasses.Add(Class);
	}
}

void FGraphNodeClassHelper::SetForcedHiddenClasses(const TSet<UClass*>& Classes)
{
	ForcedHiddenClasses = Classes;
}

void FGraphNodeClassHelper::SetGatherBlueprints(const bool bGather)
{
	bGatherBlueprints = bGather;
}

#undef LOCTEXT_NAMESPACE
