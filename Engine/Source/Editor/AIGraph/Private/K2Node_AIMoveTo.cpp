// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AIMoveTo.h"

#include "Blueprint/AIAsyncTaskBlueprintProxy.h"
#include "Blueprint/AIBlueprintHelperLibrary.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AIMoveTo)

#define LOCTEXT_NAMESPACE "K2Node_AIMoveTo"

UK2Node_AIMoveTo::UK2Node_AIMoveTo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UAIBlueprintHelperLibrary, CreateMoveToProxyObject);
	ProxyFactoryClass = UAIBlueprintHelperLibrary::StaticClass();
	ProxyClass = UAIAsyncTaskBlueprintProxy::StaticClass();
}

FText UK2Node_AIMoveTo::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::AI);
}

FText UK2Node_AIMoveTo::GetTooltipText() const
{
	return LOCTEXT("AIMoveTo_Tooltip", "Simple order for Pawn with AIController to move to a specific location.\n\n"
		"Inherits bAllowStrafe setting from AIController.\n\n"
		"Only performs strafe when Target Actor is provided.");
}

FText UK2Node_AIMoveTo::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AIMoveTo", "AI MoveTo");
}

#undef LOCTEXT_NAMESPACE


