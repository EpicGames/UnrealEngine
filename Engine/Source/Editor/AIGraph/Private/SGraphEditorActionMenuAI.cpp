// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphEditorActionMenuAI.h"

#include "AIGraphNode.h"
#include "AIGraphSchema.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SGraphActionMenu.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "Widgets/Layout/SBox.h"

SGraphEditorActionMenuAI::~SGraphEditorActionMenuAI()
{
	OnClosedCallback.ExecuteIfBound();
}

void SGraphEditorActionMenuAI::Construct( const FArguments& InArgs )
{
	this->GraphObj = InArgs._GraphObj;
	this->GraphNode = InArgs._GraphNode;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->AutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	this->SubNodeFlags = InArgs._SubNodeFlags;

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5.f)
		[
			// Achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
			.WidthOverride(400.f)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(this, &SGraphEditorActionMenuAI::OnActionSelected)
				.OnCollectAllActions(this, &SGraphEditorActionMenuAI::CollectAllActions)
				.AutoExpandActionMenu(AutoExpandActionMenu)
			]
		]
	);
}

void SGraphEditorActionMenuAI::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	// Build up the context object
	FGraphContextMenuBuilder ContextMenuBuilder(GraphObj);
	if (GraphNode != NULL)
	{
		ContextMenuBuilder.SelectedObjects.Add(GraphNode);
	}
	if (DraggedFromPins.Num() > 0)
	{
		ContextMenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Determine all possible actions
	const UAIGraphSchema* MySchema = Cast<const UAIGraphSchema>(GraphObj->GetSchema());
	if (MySchema)
	{
		MySchema->GetGraphNodeContextActions(ContextMenuBuilder, SubNodeFlags);
	}

	// Copy the added options back to the main list
	//@TODO: Avoid this copy
	OutAllActions.Append(ContextMenuBuilder);
}

TSharedRef<SEditableTextBox> SGraphEditorActionMenuAI::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}

void SGraphEditorActionMenuAI::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedAction.Num() == 0)
	{
		bool bDoDismissMenus = false;

		if (GraphObj)
		{
			for ( int32 ActionIndex = 0; ActionIndex < SelectedAction.Num(); ActionIndex++ )
			{
				TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedAction[ActionIndex];

				if ( CurrentAction.IsValid() )
				{
					CurrentAction->PerformAction(GraphObj, DraggedFromPins, UE::Slate::CastToVector2f(NewNodePosition));
					bDoDismissMenus = true;
				}
			}
		}

		if (bDoDismissMenus)
		{
			FSlateApplication::Get().DismissAllMenus();
		}
	}
}
