// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphNodeAI.h"
#include "AIGraphNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "AIGraph.h"
#include "SGraphPanel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AIGraph"

TSharedRef<FDragAIGraphNode> FDragAIGraphNode::New(const TSharedRef<SGraphPanel>& InGraphPanel, const TSharedRef<SGraphNode>& InDraggedNode)
{
	TSharedRef<FDragAIGraphNode> Operation = MakeShareable(new FDragAIGraphNode);
	Operation->StartTime = FPlatformTime::Seconds();
	Operation->GraphPanel = InGraphPanel;
	Operation->DraggedNodes.Add(InDraggedNode);
	// adjust the decorator away from the current mouse location a small amount based on cursor size
	Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();
	Operation->Construct();
	return Operation;
}

TSharedRef<FDragAIGraphNode> FDragAIGraphNode::New(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphNode> >& InDraggedNodes)
{
	TSharedRef<FDragAIGraphNode> Operation = MakeShareable(new FDragAIGraphNode);
	Operation->StartTime = FPlatformTime::Seconds();
	Operation->GraphPanel = InGraphPanel;
	Operation->DraggedNodes = InDraggedNodes;
	Operation->DecoratorAdjust = FSlateApplication::Get().GetCursorSize();
	Operation->Construct();
	return Operation;
}

UAIGraphNode* FDragAIGraphNode::GetDropTargetNode() const
{
	return Cast<UAIGraphNode>(GetHoveredNode());
}

void SGraphPinAI::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	this->SetCursor(EMouseCursor::Default);

	bShowLabel = true;

	GraphPinObj = InPin;
	check(GraphPinObj != NULL);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	check(Schema);

	SBorder::Construct(SBorder::FArguments()
		.BorderImage(this, &SGraphPinAI::GetPinBorder)
		.BorderBackgroundColor(this, &SGraphPinAI::GetPinColor)
		.OnMouseButtonDown(this, &SGraphPinAI::OnPinMouseDown)
		.Cursor(this, &SGraphPinAI::GetPinCursor)
		.Padding(FMargin(10.0f))
		);
}

TSharedRef<SWidget>	SGraphPinAI::GetDefaultValueWidget()
{
	return SNew(STextBlock);
}

const FSlateBrush* SGraphPinAI::GetPinBorder() const
{
	return FAppStyle::GetBrush(TEXT("Graph.StateNode.Body"));
}

FSlateColor SGraphPinAI::GetPinColor() const
{
	return FSlateColor(IsHovered() ? FLinearColor::Yellow : FLinearColor::Black);
}

void SGraphNodeAI::Construct(const FArguments& InArgs, UAIGraphNode* InNode)
{
	SetCursor(EMouseCursor::CardinalCross);

	GraphNode = InNode;
	UpdateGraphNode();

	bDragMarkerVisible = false;
}

void SGraphNodeAI::AddSubNode(TSharedPtr<SGraphNode> SubNodeWidget)
{
	SubNodes.Add(SubNodeWidget);
}

FText SGraphNodeAI::GetTitle() const
{
	return GraphNode ? GraphNode->GetNodeTitle(ENodeTitleType::FullTitle) : FText::GetEmpty();
}

FText SGraphNodeAI::GetDescription() const
{
	UAIGraphNode* MyNode = CastChecked<UAIGraphNode>(GraphNode);
	return MyNode ? MyNode->GetDescription() : FText::GetEmpty();
}

EVisibility SGraphNodeAI::GetDescriptionVisibility() const
{
	// LOD this out once things get too small
	TSharedPtr<SGraphPanel> MyOwnerPanel = GetOwnerPanel();
	return (!MyOwnerPanel.IsValid() || MyOwnerPanel->GetCurrentLOD() > EGraphRenderingLOD::LowDetail) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNodeAI::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = PinObj && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			[
				PinToAdd
			];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		RightNodeBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			[
				PinToAdd
			];
		OutputPins.Add(PinToAdd);
	}
}

FReply SGraphNodeAI::OnMouseMove(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && !(GEditor->bIsSimulatingInEditor || GEditor->PlayWorld))
	{
		//if we are holding mouse over a subnode
		UAIGraphNode* TestNode = Cast<UAIGraphNode>(GraphNode);
		if (TestNode && TestNode->IsSubNode())
		{
			const TSharedRef<SGraphPanel>& Panel = GetOwnerPanel().ToSharedRef();
			const TSharedRef<SGraphNode>& Node = SharedThis(this);
			return FReply::Handled().BeginDragDrop(FDragAIGraphNode::New(Panel, Node));
		}
	}

	if (!MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && bDragMarkerVisible)
	{
		SetDragMarker(false);
	}

	return FReply::Unhandled();
}

FReply SGraphNodeAI::OnMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		UAIGraphNode* TestNode = Cast<UAIGraphNode>(GraphNode);
		if (TestNode && TestNode->IsSubNode())
		{
			// First unfocus any detail panel property so that changes done to them can be applied. Normally MouseDown
			// unfocuses and MouseUp selects the new node, but since here we select instantly, we need to unfocus manually.
			FSlateApplication::Get().SetUserFocus(MouseEvent.GetUserIndex(), AsShared(), EFocusCause::Mouse);

			GetOwnerPanel()->SelectionManager.ClickedOnNode(GraphNode, MouseEvent);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

TSharedPtr<SGraphNode> SGraphNodeAI::GetSubNodeUnderCursor(const FGeometry& WidgetGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SGraphNode> ResultNode;

	// We just need to find the one WidgetToFind among our descendants.
	TSet< TSharedRef<SWidget> > SubWidgetsSet;
	for (int32 i = 0; i < SubNodes.Num(); i++)
	{
		SubWidgetsSet.Add(SubNodes[i].ToSharedRef());
	}

	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
	FindChildGeometries(WidgetGeometry, SubWidgetsSet, Result);

	if (Result.Num() > 0)
	{
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		Result.GenerateValueArray(ArrangedChildren.GetInternalArray());
		
		const int32 HoveredIndex = SWidget::FindChildUnderMouse(ArrangedChildren, MouseEvent);
		if (HoveredIndex != INDEX_NONE)
		{
			ResultNode = StaticCastSharedRef<SGraphNode>(ArrangedChildren[HoveredIndex].Widget);
		}
	}

	return ResultNode;
}

void SGraphNodeAI::SetDragMarker(bool bEnabled)
{
	bDragMarkerVisible = bEnabled;
}

EVisibility SGraphNodeAI::GetDragOverMarkerVisibility() const
{
	return bDragMarkerVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

void SGraphNodeAI::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Is someone dragging a node?
	TSharedPtr<FDragNode> DragConnectionOp = DragDropEvent.GetOperationAs<FDragNode>();
	if (DragConnectionOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are hovering over this node.
		TSharedPtr<SGraphNode> SubNode = GetSubNodeUnderCursor(MyGeometry, DragDropEvent);
		DragConnectionOp->SetHoveredNode(SubNode.IsValid() ? SubNode : SharedThis(this));

		UAIGraphNode* TestNode = Cast<UAIGraphNode>(GraphNode);
		if (DragConnectionOp->IsValidOperation() && TestNode && TestNode->IsSubNode())
		{
			SetDragMarker(true);
		}
	}

	SGraphNode::OnDragEnter(MyGeometry, DragDropEvent);
}

FReply SGraphNodeAI::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// Is someone dragging a node?
	TSharedPtr<FDragNode> DragConnectionOp = DragDropEvent.GetOperationAs<FDragNode>();
	if (DragConnectionOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are hovering over this node.
		TSharedPtr<SGraphNode> SubNode = GetSubNodeUnderCursor(MyGeometry, DragDropEvent);
		DragConnectionOp->SetHoveredNode(SubNode.IsValid() ? SubNode : SharedThis(this));
	}
	return SGraphNode::OnDragOver(MyGeometry, DragDropEvent);
}

void SGraphNodeAI::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragNode> DragConnectionOp = DragDropEvent.GetOperationAs<FDragNode>();
	if (DragConnectionOp.IsValid())
	{
		// Inform the Drag and Drop operation that we are not hovering any pins
		DragConnectionOp->SetHoveredNode(TSharedPtr<SGraphNode>(NULL));
	}

	SetDragMarker(false);
	SGraphNode::OnDragLeave(DragDropEvent);
}

FReply SGraphNodeAI::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SetDragMarker(false);

	TSharedPtr<FDragAIGraphNode> DragNodeOp = DragDropEvent.GetOperationAs<FDragAIGraphNode>();
	if (DragNodeOp.IsValid())
	{
		if (!DragNodeOp->IsValidOperation())
		{
			return FReply::Handled();
		}

		const float DragTime = float(FPlatformTime::Seconds() - DragNodeOp->StartTime);
		if (DragTime < 0.25f)
		{
			return FReply::Handled();
		}

		UAIGraphNode* MyNode = Cast<UAIGraphNode>(GraphNode);
		if (MyNode == nullptr || MyNode->IsSubNode())
		{
			return FReply::Unhandled();
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_DragDropNode", "Drag&Drop Node"));
		bool bReorderOperation = true;

		const TArray< TSharedRef<SGraphNode> >& DraggedNodes = DragNodeOp->GetNodes();
		for (int32 Idx = 0; Idx < DraggedNodes.Num(); Idx++)
		{
			UAIGraphNode* DraggedNode = Cast<UAIGraphNode>(DraggedNodes[Idx]->GetNodeObj());
			if (DraggedNode && DraggedNode->ParentNode)
			{
				if (DraggedNode->ParentNode != GraphNode)
				{
					bReorderOperation = false;
				}

				DraggedNode->ParentNode->RemoveSubNode(DraggedNode);
			}
		}

		UAIGraphNode* DropTargetNode = DragNodeOp->GetDropTargetNode();
		const int32 InsertIndex = MyNode->FindSubNodeDropIndex(DropTargetNode);

		for (int32 Idx = 0; Idx < DraggedNodes.Num(); Idx++)
		{
			UAIGraphNode* DraggedTestNode = Cast<UAIGraphNode>(DraggedNodes[Idx]->GetNodeObj());
			DraggedTestNode->Modify();
			DraggedTestNode->ParentNode = MyNode;

			MyNode->Modify();
			MyNode->InsertSubNodeAt(DraggedTestNode, InsertIndex);
		}

		if (bReorderOperation)
		{
			UpdateGraphNode();
		}
		else
		{
			UAIGraph* MyGraph = MyNode->GetAIGraph();
			if (MyGraph)
			{
				MyGraph->OnSubNodeDropped();
			}
		}
	}

	return SGraphNode::OnDrop(MyGeometry, DragDropEvent);
}

TSharedPtr<SToolTip> SGraphNodeAI::GetComplexTooltip()
{
	return NULL;
}

FText SGraphNodeAI::GetPreviewCornerText() const
{
	return FText::GetEmpty();
}

const FSlateBrush* SGraphNodeAI::GetNameIcon() const
{
	return FAppStyle::GetBrush(TEXT("Graph.StateNode.Icon"));
}

void SGraphNodeAI::SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel)
{
	SGraphNode::SetOwner(OwnerPanel);

	for (auto& ChildWidget : SubNodes)
	{
		if (ChildWidget.IsValid())
		{
			ChildWidget->SetOwner(OwnerPanel);
			OwnerPanel->AttachGraphEvents(ChildWidget);
		}
	}
}

#undef LOCTEXT_NAMESPACE
