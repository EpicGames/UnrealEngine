// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Layout/ArrangedWidget.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API AIGRAPH_API

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FGeometry;

class FAIGraphConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	UEdGraph* GraphObj;
	TMap<UEdGraphNode*, int32> NodeWidgetMap;

public:
	UE_API FAIGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FConnectionDrawingPolicy interface 
	UE_API virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	UE_API virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& PinGeometries, FArrangedChildren& ArrangedNodes) override;
	UE_API virtual void DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params) override;
	UE_API virtual void DrawSplineWithArrow(const FVector2f& StartPoint, const FVector2f& EndPoint, const FConnectionParams& Params) override;
	UE_API virtual void DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin) override;
	UE_API virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override;
	// End of FConnectionDrawingPolicy interface

protected:
	UE_API void Internal_DrawLineWithArrow(const UE::Slate::FDeprecateVector2DParameter& StartAnchorPoint, const UE::Slate::FDeprecateVector2DParameter& EndAnchorPoint, const FConnectionParams& Params);
};

#undef UE_API
