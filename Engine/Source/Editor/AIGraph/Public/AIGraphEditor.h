// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"

#define UE_API AIGRAPH_API

class FAIGraphEditor : public FEditorUndoClient
{
public:
	UE_API FAIGraphEditor();
	UE_API virtual ~FAIGraphEditor();

	UE_API FGraphPanelSelectionSet GetSelectedNodes() const;
	UE_API virtual void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);

	//~ Begin FEditorUndoClient Interface
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	UE_API void CreateCommandList();

	// Delegates for graph editor commands
	UE_API void SelectAllNodes();
	UE_API bool CanSelectAllNodes() const;
	UE_API void DeleteSelectedNodes();
	UE_API bool CanDeleteNodes() const;
	UE_API void DeleteSelectedDuplicatableNodes();
	UE_API void CutSelectedNodes();
	UE_API bool CanCutNodes() const;
	UE_API void CopySelectedNodes();
	UE_API bool CanCopyNodes() const;
	UE_API void PasteNodes();
	UE_API void PasteNodesHere(const FVector2D& Location);
	UE_API bool CanPasteNodes() const;
	UE_API void DuplicateNodes();
	UE_API bool CanDuplicateNodes() const;

	UE_API bool CanCreateComment() const;
	UE_API void OnCreateComment();


	UE_API virtual void OnClassListUpdated();

protected:
	UE_API virtual void FixupPastedNodes(const TSet<UEdGraphNode*>& NewPastedGraphNodes, const TMap<FGuid/*New*/, FGuid/*Old*/>& NewToOldNodeMapping);

protected:

	/** Currently focused graph */
	TWeakPtr<SGraphEditor> UpdateGraphEdPtr;

	/** The command list for this editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/** Handle to the registered OnClassListUpdated delegate */
	FDelegateHandle OnClassListUpdatedDelegateHandle;
};

#undef UE_API
