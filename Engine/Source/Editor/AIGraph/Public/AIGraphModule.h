// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

AIGRAPH_API DECLARE_LOG_CATEGORY_EXTERN(LogAIGraph, Display, All);

class FAIGraphModule : public IModuleInterface
{
public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
