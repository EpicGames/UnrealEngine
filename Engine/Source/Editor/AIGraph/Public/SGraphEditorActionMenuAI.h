// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "HAL/PlatformMath.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"

#define UE_API AIGRAPH_API

class SEditableTextBox;
class SGraphActionMenu;
class UAIGraphNode;
class UEdGraph;
class UEdGraphPin;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

/////////////////////////////////////////////////////////////////////////////////////////////////

class SGraphEditorActionMenuAI : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SGraphEditorActionMenuAI)
		: _GraphObj( static_cast<UEdGraph*>(NULL) )
		,_GraphNode(NULL)
		, _NewNodePosition( FVector2D::ZeroVector )
		, _AutoExpandActionMenu( false )
		, _SubNodeFlags(0)
		{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( UAIGraphNode*, GraphNode)
		SLATE_ARGUMENT( FVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
		SLATE_ARGUMENT( int32, SubNodeFlags)
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs );

	UE_API ~SGraphEditorActionMenuAI();

	UE_API TSharedRef<SEditableTextBox> GetFilterTextBox();

protected:
	UEdGraph* GraphObj;
	UAIGraphNode* GraphNode;
	TArray<UEdGraphPin*> DraggedFromPins;
	FVector2D NewNodePosition;
	bool AutoExpandActionMenu;
	int32 SubNodeFlags;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	UE_API void OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType );

	/** Callback used to populate all actions list in SGraphActionMenu */
	UE_API void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
};

#undef UE_API
