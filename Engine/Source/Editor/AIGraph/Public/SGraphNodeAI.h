// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "Editor/GraphEditor/Private/DragNode.h"

#define UE_API AIGRAPH_API

class SGraphPanel;
class SToolTip;
class UAIGraphNode;

class FDragAIGraphNode : public FDragNode
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragAIGraphNode, FDragNode)

	static TSharedRef<FDragAIGraphNode> New(const TSharedRef<SGraphPanel>& InGraphPanel, const TSharedRef<SGraphNode>& InDraggedNode);
	static TSharedRef<FDragAIGraphNode> New(const TSharedRef<SGraphPanel>& InGraphPanel, const TArray< TSharedRef<SGraphNode> >& InDraggedNodes);

	UAIGraphNode* GetDropTargetNode() const;

	double StartTime;

protected:
	typedef FDragNode Super;
};

class SGraphNodeAI : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAI){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UAIGraphNode* InNode);

	//~ Begin SGraphNode Interface
	UE_API virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void SetOwner(const TSharedRef<SGraphPanel>& OwnerPanel) override;
	UE_API virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	//~ End SGraphNode Interface

	/** handle mouse down on the node */
	UE_API FReply OnMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);

	/** adds subnode widget inside current node */
	UE_API virtual void AddSubNode(TSharedPtr<SGraphNode> SubNodeWidget);

	/** gets decorator or service node if one is found under mouse cursor */
	UE_API TSharedPtr<SGraphNode> GetSubNodeUnderCursor(const FGeometry& WidgetGeometry, const FPointerEvent& MouseEvent);

	/** gets drag over marker visibility */
	UE_API EVisibility GetDragOverMarkerVisibility() const;

	/** sets drag marker visible or collapsed on this node */
	UE_API void SetDragMarker(bool bEnabled);

protected:
	TArray< TSharedPtr<SGraphNode> > SubNodes;

	uint32 bDragMarkerVisible : 1;

	UE_API virtual FText GetTitle() const;
	UE_API virtual FText GetDescription() const;
	UE_API virtual EVisibility GetDescriptionVisibility() const;

	UE_API virtual FText GetPreviewCornerText() const;
	UE_API virtual const FSlateBrush* GetNameIcon() const;
};

class SGraphPinAI : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinAI){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InPin);
protected:
	//~ Begin SGraphPin Interface
	UE_API virtual FSlateColor GetPinColor() const override;
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API const FSlateBrush* GetPinBorder() const;
};

#undef UE_API
